// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditableMeshAdapter.h"
#include "EditableMeshCustomVersion.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryHitTest.h"
#include "EditableGeometryCollectionAdapter.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGeometryCollectionAdapter, Verbose, All);


USTRUCT()
struct FAdaptorTriangleID : public FElementID
{
	GENERATED_BODY()

	FAdaptorTriangleID()
	{
	}

	explicit FAdaptorTriangleID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	explicit FAdaptorTriangleID( const uint32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	FORCEINLINE friend uint32 GetTypeHash( const FAdaptorTriangleID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}

	/** Invalid triangle ID */
	static const FAdaptorTriangleID Invalid;
};


USTRUCT()
struct FAdaptorPolygon
{
	GENERATED_BODY()

	/** Which rendering polygon group the polygon is in */
	UPROPERTY()
	FPolygonGroupID PolygonGroupID;

	/** This is a list of indices of triangles in the FRenderingPolygon2Group2::Triangles array.
	    We use this to maintain a record of which triangles in the section belong to this polygon. */
	UPROPERTY()
	TArray<FAdaptorTriangleID> TriangulatedPolygonTriangleIndices;

	/** Serializer */
	friend FArchive& operator<<( FArchive& Ar, FAdaptorPolygon& Polygon )
	{
		Ar << Polygon.PolygonGroupID;
		Ar << Polygon.TriangulatedPolygonTriangleIndices;
		return Ar;
	}
};


USTRUCT()
struct FAdaptorPolygon2Group
{
	GENERATED_BODY()

	/** The rendering section index for this mesh section */
	UPROPERTY()
	uint32 RenderingSectionIndex=0;

	/** The material slot index assigned to this polygon group's material */
	UPROPERTY()
	int32 MaterialIndex=0;

	/** Maximum number of triangles which have been reserved in the index buffer */
	UPROPERTY()
	int32 MaxTriangles=0;

	/** Sparse array of triangles, that matches the triangles in the mesh index buffers.  Elements that
	    aren't allocated will be stored as degenerates in the mesh index buffer. */
	TMeshElementArray<FMeshTriangle, FAdaptorTriangleID> Triangles;

	/** Serializer */
	friend FArchive& operator<<( FArchive& Ar, FAdaptorPolygon2Group& Section )
	{
		Ar << Section.RenderingSectionIndex;
		Ar << Section.MaterialIndex;
		Ar << Section.MaxTriangles;	// @todo mesheditor serialization: Should not need to be serialized if we triangulate after load
		Ar << Section.Triangles;
		return Ar;
	}
};


UCLASS(MinimalAPI)
class UEditableGeometryCollectionAdapter : public UEditableMeshAdapter
{
	GENERATED_BODY()

public:

	/** Default constructor that initializes good defaults for UEditableGeometryCollectionAdapter */
	UEditableGeometryCollectionAdapter();

	virtual void Serialize( FArchive& Ar ) override;

	/** Creates a editable mesh from the specified component and sub-mesh address */
	void InitEditableGeometryCollection( UEditableMesh* EditableMesh, class UPrimitiveComponent& Component, const FEditableMeshSubMeshAddress& InitSubMeshAddress );
	EDITABLEMESH_API void InitFromBlankGeometryCollection( UEditableMesh* EditableMesh, UGeometryCollection& InGeometryCollection);

	virtual void InitializeFromEditableMesh( const UEditableMesh* EditableMesh ) override;
	virtual void OnRebuildRenderMeshStart( const UEditableMesh* EditableMesh, const bool bInvalidateLighting ) override;
	virtual void OnRebuildRenderMesh( const UEditableMesh* EditableMesh ) override;
	virtual void OnRebuildRenderMeshFinish( const UEditableMesh* EditableMesh, const bool bRebuildBoundsAndCollision, const bool bIsPreviewRollback ) override;
	virtual void OnStartModification( const UEditableMesh* EditableMesh, const EMeshModificationType MeshModificationType, const EMeshTopologyChange MeshTopologyChange ) override;
	virtual void OnEndModification( const UEditableMesh* EditableMesh ) override;
	virtual void OnReindexElements( const UEditableMesh* EditableMesh, const FElementIDRemappings& Remappings ) override;
	virtual bool IsCommitted( const UEditableMesh* EditableMesh ) const override;
	virtual bool IsCommittedAsInstance( const UEditableMesh* EditableMesh ) const override;
	virtual void OnCommit( UEditableMesh* EditableMesh ) override;
	virtual UEditableMesh* OnCommitInstance( UEditableMesh* EditableMesh, UPrimitiveComponent* ComponentToInstanceTo ) override;
	virtual void OnRevert( UEditableMesh* EditableMesh ) override;
	virtual UEditableMesh* OnRevertInstance( UEditableMesh* EditableMesh ) override;
	virtual void OnPropagateInstanceChanges( UEditableMesh* EditableMesh ) override;

	virtual void OnDeleteVertexInstances( const UEditableMesh* EditableMesh, const TArray<FVertexInstanceID>& VertexInstanceIDs ) override;
	virtual void OnDeleteOrphanVertices( const UEditableMesh* EditableMesh, const TArray<FVertexID>& VertexIDs ) override;
	virtual void OnCreateEmptyVertexRange( const UEditableMesh* EditableMesh, const TArray<FVertexID>& VertexIDs ) override;
	virtual void OnCreateVertices( const UEditableMesh* EditableMesh, const TArray<FVertexID>& VertexIDs ) override;
	virtual void OnCreateVertexInstances( const UEditableMesh* EditableMesh, const TArray<FVertexInstanceID>& VertexInstanceIDs ) override;
	virtual void OnSetVertexAttribute( const UEditableMesh* EditableMesh, const FVertexID VertexID, const FMeshElementAttributeData& Attribute ) override;
	virtual void OnSetVertexInstanceAttribute( const UEditableMesh* EditableMesh, const FVertexInstanceID VertexInstanceID, const FMeshElementAttributeData& Attribute ) override;
	virtual void OnCreateEdges( const UEditableMesh* EditableMesh, const TArray<FEdgeID>& EdgeIDs ) override;
	virtual void OnDeleteEdges( const UEditableMesh* EditableMesh, const TArray<FEdgeID>& EdgeIDs ) override;
	virtual void OnSetEdgesVertices( const UEditableMesh* EditableMesh, const TArray<FEdgeID>& EdgeIDs ) override;
	virtual void OnSetEdgeAttribute( const UEditableMesh* EditableMesh, const FEdgeID EdgeID, const FMeshElementAttributeData& Attribute ) override;
	virtual void OnCreatePolygons( const UEditableMesh* EditableMesh, const TArray<FPolygonID>& PolygonIDs ) override;
	virtual void OnDeletePolygons( const UEditableMesh* EditableMesh, const TArray<FPolygonID>& PolygonIDs ) override;
	virtual void OnChangePolygonVertexInstances( const UEditableMesh* EditableMesh, const TArray<FPolygonID>& PolygonIDs ) override;
	virtual void OnSetPolygonAttribute( const UEditableMesh* EditableMesh, const FPolygonID PolygonID, const FMeshElementAttributeData& Attribute ) override {}
	virtual void OnCreatePolygonGroups( const UEditableMesh* EditableMesh, const TArray<FPolygonGroupID>& PolygonGroupIDs ) override;
	virtual void OnDeletePolygonGroups( const UEditableMesh* EditableMesh, const TArray<FPolygonGroupID>& PolygonGroupIDs ) override;
	virtual void OnSetPolygonGroupAttribute( const UEditableMesh* EditableMesh, const FPolygonGroupID PolygonGroupID, const FMeshElementAttributeData& Attribute ) override;
	virtual void OnAssignPolygonsToPolygonGroups( const UEditableMesh* EditableMesh, const TArray<FPolygonGroupForPolygon>& PolygonGroupForPolygons ) override;
	virtual void OnRetriangulatePolygons( const UEditableMesh* EditableMesh, const TArray<FPolygonID>& PolygonIDs ) override;

#if WITH_EDITOR
	virtual void GeometryHitTest(const FHitParamsIn& InParams, FHitParamsOut& OutParams) override;
#endif // WITH_EDITOR

private:


	/** Deletes all of a polygon's triangles (including rendering triangles from the index buffer.) */
	void DeletePolygonTriangles( const UEditableMesh* EditableMesh, const FPolygonID PolygonID );

	/** Rebuilds bounds */
	void UpdateBounds( const UEditableMesh* EditableMesh, const bool bShouldRecompute );

	/** Rebuilds collision.  Bounds should always be updated first. */
	void UpdateCollision();

	/** Gets the editable mesh section index which corresponds to the given rendering section index */
	FPolygonGroupID GetSectionForRenderingSectionIndex( const int32 RenderingSectionIndex ) const;

private:

	/** The Geometry Collection asset we're representing */
	UPROPERTY()
	UGeometryCollection* GeometryCollection;

	UPROPERTY()
	UGeometryCollection* OriginalGeometryCollection;

	UPROPERTY()
	int32 GeometryCollectionLODIndex;

	/** All of the polygons in this mesh */
	TMeshElementArray<FAdaptorPolygon, FPolygonID> RenderingPolygons;

	/** All of the polygon groups in this mesh */
	TMeshElementArray<FAdaptorPolygon2Group, FPolygonGroupID> RenderingPolygonGroups;

	/** The Component this adaptir represents */
	class UGeometryCollectionComponent* GeometryCollectionComponent;

	/** Cached bounding box for the mesh.  This bounds can be (temporarily) larger than the actual mesh itself as
	    an optimization. */
	FBoxSphereBounds CachedBoundingBoxAndSphere;

	void LogGeometryCollectionStats(const FString& SourceString);

};
