// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditableMeshAdapter.h"
#include "EditableMeshCustomVersion.h"
#include "StaticMeshResources.h"
#include "EditableStaticMeshAdapter.generated.h"


USTRUCT()
struct FRenderingPolygon
{
	GENERATED_BODY()

	/** Which rendering polygon group the polygon is in */
	UPROPERTY()
	FPolygonGroupID PolygonGroupID;

	/** This is a list of indices of triangles in the FRenderingPolygonGroup::Triangles array.
	    We use this to maintain a record of which triangles in the section belong to this polygon. */
	UPROPERTY()
	TArray<FTriangleID> TriangulatedPolygonTriangleIndices;

	/** Serializer */
	friend FArchive& operator<<( FArchive& Ar, FRenderingPolygon& Polygon )
	{
		Ar << Polygon.PolygonGroupID;
		Ar << Polygon.TriangulatedPolygonTriangleIndices;
		return Ar;
	}
};


USTRUCT()
struct FRenderingPolygonGroup
{
	GENERATED_BODY()

	/** The rendering section index for this mesh section */
	UPROPERTY()
	uint32 RenderingSectionIndex=0;

	/** The material slot index assigned to this polygon group's material */
	UPROPERTY()
	int32 MaterialIndex=0;

	/** Maximum number of triangles which have been reserved in the index buffer */
	UPROPERTY()
	int32 MaxTriangles=0;

	/** Sparse array of triangles, that matches the triangles in the mesh index buffers.  Elements that
	    aren't allocated will be stored as degenerates in the mesh index buffer. */
	TMeshElementArray<FMeshTriangle, FTriangleID> Triangles;


	/** Converts from an index in our Triangles array to an index of a rendering triangle's first vertex in the rendering mesh's index buffer */
	inline static FTriangleID RenderingTriangleFirstIndexToTriangleIndex( const FStaticMeshSection& RenderingSection, const uint32 RenderingTriangleFirstIndex )
	{
		return FTriangleID( ( RenderingTriangleFirstIndex - RenderingSection.FirstIndex ) / 3 );
	}

	/** Converts from an index of a rendering triangle's first vertex in the rendering mesh's index buffer to an index in our Triangles array */
	inline static uint32 TriangleIndexToRenderingTriangleFirstIndex( const FStaticMeshSection& RenderingSection, const FTriangleID TriangleIndex )
	{
		return TriangleIndex.GetValue() * 3 + RenderingSection.FirstIndex;
	}

	/** Serializer */
	friend FArchive& operator<<( FArchive& Ar, FRenderingPolygonGroup& Section )
	{
		Ar << Section.RenderingSectionIndex;
		Ar << Section.MaterialIndex;
		Ar << Section.MaxTriangles;	// @todo mesheditor serialization: Should not need to be serialized if we triangulate after load
		Ar << Section.Triangles;
		return Ar;
	}
};




UCLASS(MinimalAPI)
class UEditableStaticMeshAdapter : public UEditableMeshAdapter
{
	GENERATED_BODY()

public:

	/** Default constructor that initializes good defaults for UEditableStaticMeshAdapter */
	UEditableStaticMeshAdapter();

	virtual void Serialize( FArchive& Ar ) override;

	/** Creates a editable static mesh from the specified component and sub-mesh address */
	void InitEditableStaticMesh( UEditableMesh* EditableMesh, class UPrimitiveComponent& Component, const FEditableMeshSubMeshAddress& InitSubMeshAddress );
	EDITABLEMESH_API void InitFromBlankStaticMesh( UEditableMesh* EditableMesh, UStaticMesh& InStaticMesh );

	virtual void InitializeFromEditableMesh( const UEditableMesh* EditableMesh ) override;
	virtual void OnRebuildRenderMeshStart( const UEditableMesh* EditableMesh, const bool bInvalidateLighting ) override;
	virtual void OnRebuildRenderMesh( const UEditableMesh* EditableMesh ) override;
	virtual void OnRebuildRenderMeshFinish( const UEditableMesh* EditableMesh, const bool bRebuildBoundsAndCollision, const bool bIsPreviewRollback ) override;
	virtual void OnStartModification( const UEditableMesh* EditableMesh, const EMeshModificationType MeshModificationType, const EMeshTopologyChange MeshTopologyChange ) override;
	virtual void OnEndModification( const UEditableMesh* EditableMesh ) override;
	virtual void OnReindexElements( const UEditableMesh* EditableMesh, const FElementIDRemappings& Remappings ) override;
	virtual bool IsCommitted( const UEditableMesh* EditableMesh ) const override;
	virtual bool IsCommittedAsInstance( const UEditableMesh* EditableMesh ) const override;
	virtual void OnCommit( UEditableMesh* EditableMesh ) override;
	virtual UEditableMesh* OnCommitInstance( UEditableMesh* EditableMesh, UPrimitiveComponent* ComponentToInstanceTo ) override;
	virtual void OnRevert( UEditableMesh* EditableMesh ) override;
	virtual UEditableMesh* OnRevertInstance( UEditableMesh* EditableMesh ) override;
	virtual void OnPropagateInstanceChanges( UEditableMesh* EditableMesh ) override;

	virtual void OnDeleteVertexInstances( const UEditableMesh* EditableMesh, const TArray<FVertexInstanceID>& VertexInstanceIDs ) override;
	virtual void OnDeleteOrphanVertices( const UEditableMesh* EditableMesh, const TArray<FVertexID>& VertexIDs ) override;
	virtual void OnCreateEmptyVertexRange( const UEditableMesh* EditableMesh, const TArray<FVertexID>& VertexIDs ) override;
	virtual void OnCreateVertices( const UEditableMesh* EditableMesh, const TArray<FVertexID>& VertexIDs ) override;
	virtual void OnCreateVertexInstances( const UEditableMesh* EditableMesh, const TArray<FVertexInstanceID>& VertexInstanceIDs ) override;
	virtual void OnSetVertexAttribute( const UEditableMesh* EditableMesh, const FVertexID VertexID, const FMeshElementAttributeData& Attribute ) override;
	virtual void OnSetVertexInstanceAttribute( const UEditableMesh* EditableMesh, const FVertexInstanceID VertexInstanceID, const FMeshElementAttributeData& Attribute ) override;
	virtual void OnCreateEdges( const UEditableMesh* EditableMesh, const TArray<FEdgeID>& EdgeIDs ) override;
	virtual void OnDeleteEdges( const UEditableMesh* EditableMesh, const TArray<FEdgeID>& EdgeIDs ) override;
	virtual void OnSetEdgesVertices( const UEditableMesh* EditableMesh, const TArray<FEdgeID>& EdgeIDs ) override;
	virtual void OnSetEdgeAttribute( const UEditableMesh* EditableMesh, const FEdgeID EdgeID, const FMeshElementAttributeData& Attribute ) override;
	virtual void OnCreatePolygons( const UEditableMesh* EditableMesh, const TArray<FPolygonID>& PolygonIDs ) override;
	virtual void OnDeletePolygons( const UEditableMesh* EditableMesh, const TArray<FPolygonID>& PolygonIDs ) override;
	virtual void OnChangePolygonVertexInstances( const UEditableMesh* EditableMesh, const TArray<FPolygonID>& PolygonIDs ) override;
	virtual void OnSetPolygonAttribute( const UEditableMesh* EditableMesh, const FPolygonID PolygonID, const FMeshElementAttributeData& Attribute ) override {}
	virtual void OnCreatePolygonGroups( const UEditableMesh* EditableMesh, const TArray<FPolygonGroupID>& PolygonGroupIDs ) override;
	virtual void OnDeletePolygonGroups( const UEditableMesh* EditableMesh, const TArray<FPolygonGroupID>& PolygonGroupIDs ) override;
	virtual void OnSetPolygonGroupAttribute( const UEditableMesh* EditableMesh, const FPolygonGroupID PolygonGroupID, const FMeshElementAttributeData& Attribute ) override;
	virtual void OnAssignPolygonsToPolygonGroups( const UEditableMesh* EditableMesh, const TArray<FPolygonGroupForPolygon>& PolygonGroupForPolygons ) override;
	virtual void OnRetriangulatePolygons( const UEditableMesh* EditableMesh, const TArray<FPolygonID>& PolygonIDs ) override;

#if WITH_EDITOR
	virtual void GeometryHitTest(const FHitParamsIn& InParams, FHitParamsOut& OutParams) override;
#endif // WITH_EDITOR

	void SetRecreateSimpleCollision(bool bInRecreateSimplifiedCollision) { bRecreateSimplifiedCollision = bInRecreateSimplifiedCollision; }
	bool GetRecreateSimpleCollision() const { return bRecreateSimplifiedCollision;  }

private:


	/** Deletes all of a polygon's triangles (including rendering triangles from the index buffer.) */
	void DeletePolygonTriangles( const UEditableMesh* EditableMesh, const FPolygonID PolygonID );

	/** Helper function to get a specific static mesh LOD structure for a mesh */
	const struct FStaticMeshLODResources& GetStaticMeshLOD() const;
	struct FStaticMeshLODResources& GetStaticMeshLOD();

	/** Makes sure our mesh's index buffer is 32-bit, converting if needed */
	void EnsureIndexBufferIs32Bit();

	/** Rebuilds bounds */
	void UpdateBounds( const UEditableMesh* EditableMesh, const bool bShouldRecompute );

	/** Rebuilds collision.  Bounds should always be updated first. */
	void UpdateCollision();

	/** Gets the editable mesh section index which corresponds to the given rendering section index */
	FPolygonGroupID GetSectionForRenderingSectionIndex( const int32 RenderingSectionIndex ) const;

private:

	/** The static mesh asset we're representing */
	UPROPERTY()
	UStaticMesh* StaticMesh;

	UPROPERTY()
	UStaticMesh* OriginalStaticMesh;

	UPROPERTY()
	int32 StaticMeshLODIndex;

	/** Size of the extra gaps in the index buffer between different sections (so new triangles have some space to be added into,
		without requiring the index buffer to be manipulated) */
	static const uint32 IndexBufferInterSectionGap = 32;

	/** All of the polygons in this mesh */
	TMeshElementArray<FRenderingPolygon, FPolygonID> RenderingPolygons;

	/** All of the polygon groups in this mesh */
	TMeshElementArray<FRenderingPolygonGroup, FPolygonGroupID> RenderingPolygonGroups;

	/** Used to refresh all components in the scene that may be using a mesh we're editing */
	TSharedPtr<class FStaticMeshComponentRecreateRenderStateContext> RecreateRenderStateContext;

	/** Cached bounding box for the static mesh.  This bounds can be (temporarily) larger than the actual mesh itself as
	    an optimization. */
	FBoxSphereBounds CachedBoundingBoxAndSphere;

	/** Flag to indicate if a modification requires regenerating collision */
	bool bUpdateCollisionNeeded;

	/** Flag to indicate if the simple collision has to be regenerated */
	bool bRecreateSimplifiedCollision;
};
