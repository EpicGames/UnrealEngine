// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "MeshTypes.h"
#include "MeshAttributeArray.h"
#include "Materials/MaterialInterface.h"
#include "EditableMeshTypes.generated.h"


// @todo mesheditor: Move elsewhere
namespace LogHelpers
{
	template <typename ElementType, typename TEnableIf<!TIsArithmetic<ElementType>::Value, int>::Type = 0>
	inline FString ElementToString( const ElementType& Element )
	{
		return Element.ToString();
	}

	template <typename ElementType, typename TEnableIf<TIsArithmetic<ElementType>::Value, int>::Type = 0>
	inline FString ElementToString( const ElementType& Element )
	{
		return LexToString( Element );
	}

	template <typename ArrayType>
	inline FString ArrayToString( const TArray<ArrayType>& Array )
	{
		FString String;
		if( Array.Num() == 0 )
		{
			String = TEXT( "Empty" );
		}
		else
		{
			for( const ArrayType& Element : Array )
			{
				if( !String.IsEmpty() )
				{
					String.Append( TEXT( ", " ) );
				}
				String.Append( ElementToString( Element ) );
			}
			String = TEXT( "[" ) + String + TEXT( "]" );
		}
		return String;
	}

	template<typename ArrayType>
	inline FString ArrayToString( const TArray<TArray<ArrayType>>& Array )
	{
		FString String;
		if( Array.Num() == 0 )
		{
			String = TEXT( "Empty" );
		}
		else
		{
			for( const TArray<ArrayType>& SubArray : Array )
			{
				if( !String.IsEmpty() )
				{
					String.Append( TEXT( ", " ) );
				}
				String.Append( FString::Printf( TEXT( "%s" ), *ArrayToString( SubArray ) ) );
			}
			String = TEXT( "[" ) + String + TEXT( "]" );
		}
		return String;
	}

	inline FString BoolToString( const bool Bool )
	{
		return Bool ? TEXT( "true" ) : TEXT( "false" );
	}
}


/**
 * The different components that make up a typical mesh
 */
enum class EEditableMeshElementType
{
	/** Invalid mesh element (or "none") */
	Invalid,

	/** A unique point in 3D space */
	Vertex,

	/** An edge that connects two vertices */
	Edge,

	/** A polygon with at least three 3D points.  It could be triangle, quad, or more complex shape */
	Polygon,

	/** Mesh Fracturing Mode */
	Fracture,

	/** Represents any element type */
	Any,
};


/**
 * Unique identifies a specific sub-mesh within a component
 */
struct FEditableMeshSubMeshAddress
{
	/** Pointer that uniquely identifies the mesh object being edited (not the instance), for hashing/comparison purposes */
	void* MeshObjectPtr;

	/** The type of mesh format */
	// @todo mesheditor: Should SHOULD be OK, but if it somehow was deallocated while we were using it, we'd want to store an FName or ID here instead of a pointer.  Should be fine though.
	class IEditableMeshFormat* EditableMeshFormat;

	/** The index of the mesh within the component, for components that may define more than one mesh */
	int32 MeshIndex;

	/** The mesh level of detail index, or zero if not applicable to the type of mesh. */
	int32 LODIndex;


	/** Default constructor that initializes variables to an default sub-mesh address */
	FEditableMeshSubMeshAddress()
		: MeshObjectPtr( nullptr ),
		  EditableMeshFormat( nullptr ),
		  MeshIndex( 0 ),
		  LODIndex( 0 )
	{
	}

	/** Equality check */
	inline bool operator==( const FEditableMeshSubMeshAddress& Other ) const
	{
		return
			MeshObjectPtr == Other.MeshObjectPtr &&
			EditableMeshFormat == Other.EditableMeshFormat &&
			MeshIndex == Other.MeshIndex &&
			LODIndex == Other.LODIndex;
	}

	/** Hashing */
	FORCEINLINE friend uint32 GetTypeHash( const FEditableMeshSubMeshAddress& Other )
	{
		// @todo mesheditor: Hash could be improved a bit to consider LOD/MeshIndex, etc.
		return GetTypeHash( Other.MeshObjectPtr );
	}

	/** Convert to a string */
	inline FString ToString() const
	{
		return FString::Printf(
			TEXT( "PtrHash:%lu, FmtHash:%lu, MeshIndex:%lu, LODIndex:%lu" ),
			GetTypeHash( MeshObjectPtr ),
			GetTypeHash( EditableMeshFormat ),
			MeshIndex,
			LODIndex );
	}
};


UENUM()
enum class EMeshModificationType : uint8
{
	/** The first Interim change since the last Final change.  This must be followed by either an Interim change or a Final change */
	FirstInterim,

	/** User is still in the middle of their interaction.  More changes to come, so don't bother finalizing everything yet (smoother performance) */
	Interim,

	/** User has finished their current interaction with this mesh, and everything needs to be finalized at this time */
	Final
};


UENUM()
enum class EMeshTopologyChange : uint8
{
	/** We won't be changing the mesh topology, but values could be changed (vertex positions, UVs, colors, etc.) */
	NoTopologyChange,

	/** Topology is changing with this edit, potentially along with other changes */
	TopologyChange
};



USTRUCT( BlueprintType )
struct FSubdividedQuadVertex
{
	GENERATED_BODY()

	/** The index of the vertex position (into the FSubdivisionLimitData's VertexPositions array) used for this vertex */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	int32 VertexPositionIndex=0;

	/** Texture coordinates for this vertex.  We only support up to two, for now. (Just to avoid TArrays/allocations) */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	FVector2D TextureCoordinate0= FVector2D::ZeroVector;
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	FVector2D TextureCoordinate1 = FVector2D::ZeroVector;

	/** Vertex color */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	FColor VertexColor = FColor::Black;

	/** Quad vertex normal */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	FVector VertexNormal =FVector::YAxisVector;

	/** Quad vertex tangent */
	UPROPERTY(BlueprintReadWrite, Category = "Editable Mesh")	FVector VertexTangent=FVector::ZAxisVector;

	/** Quad vertex binormal sign (-1.0 or 1.0)*/
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	float VertexBinormalSign=1.0f;
};


USTRUCT( BlueprintType )
struct FSubdividedQuad
{
	GENERATED_BODY()

	// NOTE: The reason we're using separate variables instead of a static array is so that we can expose these to Blueprints, which doesn't support static array properties

	/** The vertices for the four corners of this quad */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	FSubdividedQuadVertex QuadVertex0;
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	FSubdividedQuadVertex QuadVertex1;
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	FSubdividedQuadVertex QuadVertex2;
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	FSubdividedQuadVertex QuadVertex3;

	const FSubdividedQuadVertex& GetQuadVertex( const int32 Index ) const
	{
		switch( Index )
		{
			default:
			case 0:
				return QuadVertex0;

			case 1:
				return QuadVertex1;

			case 2:
				return QuadVertex2;
			
			case 3:
				return QuadVertex3;
		}
	}

	FSubdividedQuadVertex& AccessQuadVertex( const int32 Index )
	{
		switch( Index )
		{
			default:
			case 0:
				return QuadVertex0;

			case 1:
				return QuadVertex1;

			case 2:
				return QuadVertex2;

			case 3:
				return QuadVertex3;
		}
	}
};


USTRUCT( BlueprintType )
struct FSubdividedWireEdge
{
	GENERATED_BODY()

	// NOTE: The reason we're using separate variables instead of a static array is so that we can expose these to Blueprints, which doesn't support static array properties

	/** The vertex indices for the two corners of this quad */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	int32 EdgeVertex0PositionIndex=0;
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )	int32 EdgeVertex1PositionIndex=0;

	/** True if this edge is a counterpart to an original base cage edge of the mesh.  Otherwise it's a new edge that exists only
	    in the subdivision surfaces */
	FEdgeID CounterpartEdgeID;
};


USTRUCT( BlueprintType )
struct FSubdivisionLimitSection
{
	GENERATED_BODY()

	/** All of the quads in this section, as a result from subdividing the mesh */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FSubdividedQuad> SubdividedQuads;
};


USTRUCT( BlueprintType )
struct FSubdivisionLimitData
{
	GENERATED_BODY()

	/** Positions of all of the vertices for this subdivision level.  Many vertex positions may be shared between subdivided quads. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FVector> VertexPositions;

	/** Data for each of the sections in the mesh.  This array will have the same number of elements as the editable mesh's 
	    section list (not necessarily the same indices though, due to sparseness). */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FSubdivisionLimitSection> Sections;	

	/** All of the wire edges in the entire mesh (for all sections) */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FSubdividedWireEdge> SubdividedWireEdges;
};


UENUM()
enum class EMeshElementAttributeType : uint8
{
	None,
	FVector4,
	FVector,
	FVector2D,
	Float,
	Int,
	Bool,
	FName
};


USTRUCT( BlueprintType )
struct FMeshElementAttributeValue
{
	// @note: This should be replaced by TVariant as soon as it becomes available.
	// This is a very verbose and hardcoded variant type class.

	GENERATED_BODY()

	/** Define constructors */
	FMeshElementAttributeValue() : Type( EMeshElementAttributeType::None ) {}
	explicit FMeshElementAttributeValue( const FVector4& Value ) : Type( EMeshElementAttributeType::FVector4 ), Value_FVector4( Value ) {}
	explicit FMeshElementAttributeValue( const FVector Value ) : Type( EMeshElementAttributeType::FVector ), Value_FVector( Value ) {}
	explicit FMeshElementAttributeValue( const FVector2D Value ) : Type( EMeshElementAttributeType::FVector2D ), Value_FVector2D( Value ) {}
	explicit FMeshElementAttributeValue( const float Value ) : Type( EMeshElementAttributeType::Float ), Value_Float( Value ) {}
	explicit FMeshElementAttributeValue( const int Value ) : Type( EMeshElementAttributeType::Int ), Value_Int( Value ) {}
	explicit FMeshElementAttributeValue( const bool Value ) : Type( EMeshElementAttributeType::Bool ), Value_Bool( Value ) {}
	explicit FMeshElementAttributeValue( const FName Value ) : Type( EMeshElementAttributeType::FName ), Value_FName( Value ) {}

	/** Returns the type of this attribute value */
	EMeshElementAttributeType GetType() const
	{
		return Type;
	}

	/** Accessors for known types. There will be a runtime error if the attribute value is not of the specified type. */
	template <typename T> T GetValue() const;

	/**
	 * Calls the specified polymorphic lambda, with this attribute value as its passed-in parameter.
	 *
	 * For example:
	 *  AttributeValue.Visit( []( const auto& Value ) { DoSomethingWith( Value ); } );
	 */
	template <typename FuncType>
	void Visit( const FuncType& Func ) const
	{
		switch( Type )
		{
			case EMeshElementAttributeType::FVector4: Func( Value_FVector4 ); break;
			case EMeshElementAttributeType::FVector: Func( Value_FVector ); break;
			case EMeshElementAttributeType::FVector2D: Func( Value_FVector2D ); break;
			case EMeshElementAttributeType::Float: Func( Value_Float ); break;
			case EMeshElementAttributeType::Int: Func( Value_Int ); break;
			case EMeshElementAttributeType::Bool: Func( Value_Bool ); break;
			case EMeshElementAttributeType::FName: Func( Value_FName ); break;
		}
	}

	/** Returns the attribute value as a string */
	FString ToString() const
	{
		switch( Type )
		{
			case EMeshElementAttributeType::None: return FString( "<none>" );
			case EMeshElementAttributeType::FVector4: return Value_FVector4.ToString();
			case EMeshElementAttributeType::FVector: return Value_FVector.ToString();
			case EMeshElementAttributeType::FVector2D: return Value_FVector2D.ToString();
			case EMeshElementAttributeType::Float: return LexToString( Value_Float );
			case EMeshElementAttributeType::Int: return LexToString( Value_Int );
			case EMeshElementAttributeType::Bool: return LexToString( Value_Bool );
			case EMeshElementAttributeType::FName: return Value_FName.ToString();
		}

		return FString();
	}

	/** Serializer */
	friend FArchive& operator<<( FArchive& Ar, FMeshElementAttributeValue& AttributeValue )
	{
		Ar << AttributeValue.Type;

		switch( AttributeValue.Type )
		{
			case EMeshElementAttributeType::FVector4: Ar << AttributeValue.Value_FVector4; break;
			case EMeshElementAttributeType::FVector: Ar << AttributeValue.Value_FVector; break;
			case EMeshElementAttributeType::FVector2D: Ar << AttributeValue.Value_FVector2D; break;
			case EMeshElementAttributeType::Float: Ar << AttributeValue.Value_Float; break;
			case EMeshElementAttributeType::Int: Ar << AttributeValue.Value_Int; break;
			case EMeshElementAttributeType::Bool: Ar << AttributeValue.Value_Bool; break;
			case EMeshElementAttributeType::FName: Ar << AttributeValue.Value_FName; break;
		}

		return Ar;
	}

	/** Test equality of two attribute values */
	friend bool operator==( const FMeshElementAttributeValue& Value1, const FMeshElementAttributeValue& Value2 )
	{
		if( Value1.Type != Value2.Type )
		{
			return false;
		}

		switch( Value1.Type )
		{
			case EMeshElementAttributeType::None: return true;
			case EMeshElementAttributeType::FVector4: return Value1.Value_FVector4 == Value2.Value_FVector4;
			case EMeshElementAttributeType::FVector: return Value1.Value_FVector == Value2.Value_FVector;
			case EMeshElementAttributeType::FVector2D: return Value1.Value_FVector2D == Value2.Value_FVector2D;
			case EMeshElementAttributeType::Float: return Value1.Value_Float == Value2.Value_Float;
			case EMeshElementAttributeType::Int: return Value1.Value_Int == Value2.Value_Int;
			case EMeshElementAttributeType::Bool: return Value1.Value_Bool == Value2.Value_Bool;
			case EMeshElementAttributeType::FName: return Value1.Value_FName == Value2.Value_FName;
		}

		return false;
	}

	friend bool operator!=( const FMeshElementAttributeValue& Value1, const FMeshElementAttributeValue& Value2 )
	{
		return !( Value1 == Value2 );
	}


private:
	EMeshElementAttributeType Type;
	FVector4 Value_FVector4;
	FVector Value_FVector;
	FVector2D Value_FVector2D;
	float Value_Float;
	int Value_Int;
	bool Value_Bool;
	FName Value_FName;
};

/** Implementations of accessors for known types. These must be defined outside the class definition. */
template <> inline FVector4 FMeshElementAttributeValue::GetValue<FVector4>() const { check( Type == EMeshElementAttributeType::FVector4 ); return Value_FVector4; }
template <> inline FVector FMeshElementAttributeValue::GetValue<FVector>() const { check( Type == EMeshElementAttributeType::FVector ); return Value_FVector; }
template <> inline FVector2D FMeshElementAttributeValue::GetValue<FVector2D>() const { check( Type == EMeshElementAttributeType::FVector2D ); return Value_FVector2D; }
template <> inline float FMeshElementAttributeValue::GetValue<float>() const { check( Type == EMeshElementAttributeType::Float ); return Value_Float; }
template <> inline int FMeshElementAttributeValue::GetValue<int>() const { check( Type == EMeshElementAttributeType::Int ); return Value_Int; }
template <> inline bool FMeshElementAttributeValue::GetValue<bool>() const { check( Type == EMeshElementAttributeType::Bool ); return Value_Bool; }
template <> inline FName FMeshElementAttributeValue::GetValue<FName>() const { check( Type == EMeshElementAttributeType::FName ); return Value_FName; }



USTRUCT( BlueprintType )
struct FMeshElementAttributeData
{
	GENERATED_BODY()

	/** Name of the attribute */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FName AttributeName;

	/** Index of the attribute */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	int32 AttributeIndex;

	/** The value of this attribute */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeValue AttributeValue;

	/** Default constructor */
	FMeshElementAttributeData()
		: AttributeName( NAME_None ),
		  AttributeIndex( 0 ),
		  AttributeValue()
	{}

	/** Constructor that fills in all fields */
	FMeshElementAttributeData( const FName InitAttributeName, const int32 InitAttributeIndex, const FMeshElementAttributeValue& InitAttributeValue )
		: AttributeName( InitAttributeName ),
		  AttributeIndex( InitAttributeIndex ),
		  AttributeValue( InitAttributeValue )
	{}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "Name:%s, Index:%lu, Value:%s" ),
			*AttributeName.ToString(),
			AttributeIndex,
			*AttributeValue.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FMeshElementAttributeList
{
	GENERATED_BODY()

	/** List of attributes to apply to a mesh element */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FMeshElementAttributeData> Attributes;

	/** Default constructor */
	FMeshElementAttributeList()
		: Attributes()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "Attributes:%s" ),
			*LogHelpers::ArrayToString( Attributes ) );
	}
};


USTRUCT( BlueprintType )
struct FVertexToCreate
{
	GENERATED_BODY()

	/** Attributes of this vertex itself */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeList VertexAttributes;

	/** The original ID of the vertex.  Should only be used by the undo system. */
	UPROPERTY()
	FVertexID OriginalVertexID;

	FVertexToCreate()
		: VertexAttributes(),
		  OriginalVertexID( FVertexID::Invalid )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexAttributes:%s, OriginalVertexID:%s" ),
			*VertexAttributes.ToString(),
			*OriginalVertexID.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FVertexInstanceToCreate
{
	GENERATED_BODY()

	/** Vertex ID which is being instanced */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID VertexID;

	/** Attributes of this vertex instance */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeList VertexInstanceAttributes;

	/** The original ID of the vertex instance.  Should only be used by the undo system. */
	UPROPERTY()
	FVertexInstanceID OriginalVertexInstanceID;

	FVertexInstanceToCreate()
		: VertexID( FVertexID::Invalid ),
		  VertexInstanceAttributes(),
		  OriginalVertexInstanceID( FVertexInstanceID::Invalid )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexID:%s, VertexInstanceAttributes:%s, OriginalVertexInstanceID:%s" ),
			*VertexID.ToString(),
			*VertexInstanceAttributes.ToString(),
			*OriginalVertexInstanceID.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FEdgeToCreate
{
	GENERATED_BODY()
		
	/** The first vertex this edge connects */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID VertexID0;

	/** The second vertex this edge connects */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID VertexID1;

	///** The polygons that are connected to this edge. */
	//UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	//TArray<FPolygonID> ConnectedPolygons;

	/** Attributes of this edge itself */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeList EdgeAttributes;

	/** The original ID of the edge.  Should only be used by the undo system. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FEdgeID OriginalEdgeID;

	/** Default constructor */
	FEdgeToCreate()
		: VertexID0( FVertexID::Invalid ),
		  VertexID1( FVertexID::Invalid ),
//		  ConnectedPolygons(),
		  EdgeAttributes(),
		  OriginalEdgeID( FEdgeID::Invalid )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexID0:%s, VertexID1:%s, EdgeAttributes:%s, OriginalEdgeID:%s" ),
			*VertexID0.ToString(), 
			*VertexID1.ToString(),
//			*LogHelpers::ArrayToString( ConnectedPolygons ),
			*EdgeAttributes.ToString(),
			*OriginalEdgeID.ToString() );
	}
};



USTRUCT( BlueprintType )
struct FVertexAndAttributes
{
	GENERATED_BODY()

	/** The vertex instance ID to insert into the polygon, or FVertexInstanceID::Invalid to create a new vertex instance with the given attributes */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexInstanceID VertexInstanceID;

	/** The vertex ID to insert into the polygon, if no valid vertex instance ID was supplied. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID VertexID;

	/** A list of polygon attributes to set for the vertex on the polygon we're inserting it into, if no valid vertex instance ID was supplied. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeList PolygonVertexAttributes;

	/** Default constructor */
	FVertexAndAttributes()
		: VertexInstanceID( FVertexInstanceID::Invalid ),
		  VertexID( FVertexID::Invalid ),
		  PolygonVertexAttributes()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexInstanceID:%s, VertexID:%s, PolygonVertexAttributes:%s" ),
			*VertexInstanceID.ToString(),
			*VertexID.ToString(),
			*PolygonVertexAttributes.ToString() );
	}
};


UENUM( BlueprintType )
enum class EPolygonEdgeHardness : uint8
{
	/** Create any new edges required by the polygon as soft, leaving existing edges as they are */
	NewEdgesSoft,

	/** Create any new edges required by the polygon as hard, leaving existing edges as they are */
	NewEdgesHard,

	/** Set all edges as soft, overriding edge hardness on existing edges */
	AllEdgesSoft,

	/** Set all edges as hard, overriding edge hardness on existing edges */
	AllEdgesHard
};


USTRUCT( BlueprintType )
struct FPolygonToCreate
{
	GENERATED_BODY()

	/** The group the polygon will be added to */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FPolygonGroupID PolygonGroupID;

	/** Ordered list of vertices that defines the polygon's perimeter, along with the polygon vertex attributes to set for each vertex */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FVertexAndAttributes> PerimeterVertices;
	
	/** The original ID of the polygon.  Should only be used by the undo system. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FPolygonID OriginalPolygonID;

	/** Whether to create a hard-edged polygon */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	EPolygonEdgeHardness PolygonEdgeHardness;

	/** Default constructor */
	FPolygonToCreate()
		: PolygonGroupID( FPolygonGroupID::Invalid ),
		  PerimeterVertices(),
		  OriginalPolygonID( FPolygonID::Invalid ),
		  PolygonEdgeHardness( EPolygonEdgeHardness::NewEdgesSoft )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "PolygonGroupID:%s, PerimeterVertices:%s, OriginalPolygonID:%s" ),
			*PolygonGroupID.ToString(),
			*LogHelpers::ArrayToString( PerimeterVertices ),
			*OriginalPolygonID.ToString() );
	}
};



USTRUCT( BlueprintType )
struct FVertexPair
{
	GENERATED_BODY()

	/** The first vertex ID in this pair */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID VertexID0;

	/** The second vertex ID in this pair */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID VertexID1;

	/** Default constructor */
	FVertexPair()
		: VertexID0( FVertexID::Invalid ),
		  VertexID1( FVertexID::Invalid )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexID0:%s, VertexID1:%s" ),
			*VertexID0.ToString(),
			*VertexID1.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FPolygonToSplit
{
	GENERATED_BODY()

	/** The polygon that we'll be splitting */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FPolygonID PolygonID;

	/** A list of pairs of vertices that new edges will be created at.  The pairs must be ordered, and the vertices
	    must already exist and be connected to the polygon */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FVertexPair> VertexPairsToSplitAt;

	/** Default constructor */
	FPolygonToSplit()
		: PolygonID( FPolygonID::Invalid ),
		  VertexPairsToSplitAt()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "PolygonID:%s, VertexPairsToSplitAt:%s" ),
			*PolygonID.ToString(),
			*LogHelpers::ArrayToString( VertexPairsToSplitAt ) );
	}
};


USTRUCT( BlueprintType )
struct FAttributesForVertex
{
	GENERATED_BODY()

	/** The vertex ID to set attributes on */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID VertexID;

	/** A list of attributes to set for the vertex */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeList VertexAttributes;

	/** Default constructor */
	FAttributesForVertex()
		: VertexID( 0 ),
		  VertexAttributes()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexID:%s, VertexAttributes:%s" ),
			*VertexID.ToString(),
			*VertexAttributes.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FAttributesForVertexInstance
{
	GENERATED_BODY()

	/** The vertex instance ID to set attributes on */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexInstanceID VertexInstanceID;

	/** A list of attributes to set for the vertex instance */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeList VertexInstanceAttributes;

	/** Default constructor */
	FAttributesForVertexInstance()
		: VertexInstanceID( 0 ),
		  VertexInstanceAttributes()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexInstanceID:%s, VertexInstanceAttributes:%s" ),
			*VertexInstanceID.ToString(),
			*VertexInstanceAttributes.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FAttributesForEdge
{
	GENERATED_BODY()

	/** The edge ID to set attributes on */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FEdgeID EdgeID;

	/** A list of attributes to set for the edge */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeList EdgeAttributes;

	/** Default constructor */
	FAttributesForEdge()
		: EdgeID( 0 ),
		  EdgeAttributes()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "EdgeID:%s, EdgeAttributes:%s" ),
			*EdgeID.ToString(),
			*EdgeAttributes.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FVertexAttributesForPolygonHole
{
	GENERATED_BODY()

	/** For each hole vertex, a list of attributes for that vertex.  You can leave a given array empty for 
	    a specific hole index if you don't want to set attributes for select holes. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FMeshElementAttributeList> VertexAttributeList;

	/** Default constructor */
	FVertexAttributesForPolygonHole()
		: VertexAttributeList()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexAttributeList:%s" ),
			*LogHelpers::ArrayToString( VertexAttributeList ) );
	}
};


USTRUCT( BlueprintType )
struct FVertexAttributesForPolygon
{
	GENERATED_BODY()

	/** The polygon to set vertex attributes on */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FPolygonID PolygonID;

	/** For each polygon vertex, a list of attributes for that vertex.  Can be left empty if you don't want to set any attributes. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FMeshElementAttributeList> PerimeterVertexAttributeLists;

	/** For each hole vertex, a list of attributes for that vertex.  Can be left empty if you don't want to set any attributes.  Also
	    you can leave a given array empty for a specific hole index if you don't want to set attributes for select holes. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FVertexAttributesForPolygonHole> VertexAttributeListsForEachHole;

	/** Default constructor */
	FVertexAttributesForPolygon()
		: PolygonID( FPolygonID::Invalid ),
		  PerimeterVertexAttributeLists(),
		  VertexAttributeListsForEachHole()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "PolygonID:%s, PerimeterVertexAttributeLists:%s, VertexAttributeListsForEachHole:%s" ),
			*PolygonID.ToString(),
			*LogHelpers::ArrayToString( PerimeterVertexAttributeLists ),
			*LogHelpers::ArrayToString( VertexAttributeListsForEachHole ) );
	}
};


USTRUCT( BlueprintType )
struct FVertexIndexAndInstanceID
{
	GENERATED_BODY()

	/** Contour index of the vertex to change */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	int32 ContourIndex=0;

	/** New vertex instance ID to assign */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexInstanceID VertexInstanceID;

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "ContourIndex:%d, VertexInstanceIDs:%s" ),
			ContourIndex,
			*VertexInstanceID.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FVertexInstancesForPolygonHole
{
	GENERATED_BODY()

	/** A list of vertex instance IDs for a polygon hole. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FVertexIndexAndInstanceID> VertexIndicesAndInstanceIDs;

	/** Default constructor */
	FVertexInstancesForPolygonHole()
		: VertexIndicesAndInstanceIDs()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexInstanceIDs:%s" ),
			*LogHelpers::ArrayToString( VertexIndicesAndInstanceIDs ) );
	}
};


USTRUCT( BlueprintType )
struct FChangeVertexInstancesForPolygon
{
	GENERATED_BODY()

	/** The polygon to set vertex instances on */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FPolygonID PolygonID;

	/** A list of vertex numbers and vertex instance IDs to change for the polygon perimeter. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FVertexIndexAndInstanceID> PerimeterVertexIndicesAndInstanceIDs;

	/** A list of vertex instance IDs for each hole. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	TArray<FVertexInstancesForPolygonHole> VertexIndicesAndInstanceIDsForEachHole;

	/** Default constructor */
	FChangeVertexInstancesForPolygon()
		: PolygonID( FPolygonID::Invalid ),
		  PerimeterVertexIndicesAndInstanceIDs(),
		  VertexIndicesAndInstanceIDsForEachHole()
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "PolygonID:%s, PerimeterVertexIndicesAndInstanceIDs:%s, VertexIndicesAndInstanceIDsForEachHole:%s" ),
			*PolygonID.ToString(),
			*LogHelpers::ArrayToString( PerimeterVertexIndicesAndInstanceIDs ),
			*LogHelpers::ArrayToString( VertexIndicesAndInstanceIDsForEachHole ) );
	}
};


#if 0
USTRUCT( BlueprintType )
struct FVerticesForEdge
{
	GENERATED_BODY()

	/** The edge ID */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FEdgeID EdgeID;

	/** First new vertex ID for this edge */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID NewVertexID0;

	/** Second new vertex ID for this edge */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID NewVertexID1;

	/** Default constructor */
	FVerticesForEdge()
		: EdgeID( 0 ),
		  NewVertexID0( FVertexID::Invalid ),	// @todo mesheditor urgent: Typesafety isn't working -- able to assign EdgeID to VertexID!
		  NewVertexID1( FVertexID::Invalid )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "EdgeID:%s, NewVertexID0:%s, NewVertexID1:%s]" ),
			*EdgeID.ToString(),
			*NewVertexID0.ToString(), 
			*NewVertexID1.ToString() );
	}
};
#endif


USTRUCT( BlueprintType )
struct FVertexToMove
{
	GENERATED_BODY()

	/** The vertex we'll be moving around */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVertexID VertexID;

	/** The new position of the vertex */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FVector NewVertexPosition;


	FVertexToMove()
		: VertexID( FVertexID::Invalid ),
		  NewVertexPosition( FVector::ZeroVector )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "VertexID:%s, NewVertexPosition:%s" ),
			*VertexID.ToString(),
			*NewVertexPosition.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FPolygonGroupToCreate
{
	GENERATED_BODY()

	/** Attributes of this polygon group */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FMeshElementAttributeList PolygonGroupAttributes;

	/** The original ID of the polygon group.  Should only be used by the undo system. */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FPolygonGroupID OriginalPolygonGroupID;

	/** Default constructor */
	FPolygonGroupToCreate()
		: OriginalPolygonGroupID( FPolygonGroupID::Invalid )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "PolygonGroupAttributes:%s, OriginalPolygonGroupID:%s" ),
			*PolygonGroupAttributes.ToString(),
			*OriginalPolygonGroupID.ToString() );
	}
};


USTRUCT( BlueprintType )
struct FPolygonGroupForPolygon
{
	GENERATED_BODY()

	/** Polygon to assign to a new group */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FPolygonID PolygonID;

	/** Polygon group to assign polygon to */
	UPROPERTY( BlueprintReadWrite, Category="Editable Mesh" )
	FPolygonGroupID PolygonGroupID;

	/** Default constructor */
	FPolygonGroupForPolygon()
		: PolygonID( FPolygonID::Invalid )
		, PolygonGroupID( FPolygonGroupID::Invalid )
	{
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT( "PolygonID:%s, PolygonGroupID:%s" ),
			*PolygonID.ToString(),
			*PolygonGroupID.ToString() );
	}
};
